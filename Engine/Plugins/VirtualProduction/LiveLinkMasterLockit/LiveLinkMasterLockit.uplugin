{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkMasterLockit",
	"Description": "Live Link support for the Ambient MasterLockit metadata server",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "LiveLinkMasterLockit",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "LiveLinkMasterLockitEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}