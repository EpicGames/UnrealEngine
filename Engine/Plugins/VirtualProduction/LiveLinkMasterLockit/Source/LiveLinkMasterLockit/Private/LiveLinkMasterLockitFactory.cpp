// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMasterLockitFactory.h"

#include "LiveLinkMasterLockitSource.h"

#define LOCTEXT_NAMESPACE "LiveLinkMasterLockitSourceFactory"

ULiveLinkMasterLockitSourceFactory::FBuildCreationPanelDelegate ULiveLinkMasterLockitSourceFactory::OnBuildCreationPanel;

FText ULiveLinkMasterLockitSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "MasterLockit");
}

FText ULiveLinkMasterLockitSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Creates a connection to a MasterLockit metadata server");
}

TSharedPtr<SWidget> ULiveLinkMasterLockitSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	if (OnBuildCreationPanel.IsBound())
	{
		return OnBuildCreationPanel.Execute(this, InOnLiveLinkSourceCreated);
	}
	return TSharedPtr<SWidget>();
}

TSharedPtr<ILiveLinkSource> ULiveLinkMasterLockitSourceFactory::CreateSource(const FString& ConnectionString) const
{
	FLiveLinkMasterLockitConnectionSettings Settings;
	if (!ConnectionString.IsEmpty())
	{
		FLiveLinkMasterLockitConnectionSettings::StaticStruct()->ImportText(*ConnectionString, &Settings, nullptr, EPropertyPortFlags::PPF_None, nullptr, FLiveLinkMasterLockitConnectionSettings::StaticStruct()->GetName(), true);
	}
	return MakeShared<FLiveLinkMasterLockitSource>(Settings);
}

FString ULiveLinkMasterLockitSourceFactory::CreateConnectionString(const FLiveLinkMasterLockitConnectionSettings& Settings)
{
	FString ConnectionString;
	FLiveLinkMasterLockitConnectionSettings::StaticStruct()->ExportText(ConnectionString, &Settings, nullptr, nullptr, EPropertyPortFlags::PPF_None, nullptr, true);
	return ConnectionString;
}

#undef LOCTEXT_NAMESPACE
