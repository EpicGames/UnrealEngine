// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMasterLockitSource.h"

#include "ILiveLinkClient.h"

#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "MasterLockitMessageThread.h"
#include "Roles/LiveLinkCameraRole.h"
#include "Roles/LiveLinkCameraTypes.h"

#include "Sockets.h"

#define LOCTEXT_NAMESPACE "LiveLinkMasterLockitSource"

DEFINE_LOG_CATEGORY_STATIC(LogMasterLockitPlugin, Log, All);

FLiveLinkMasterLockitSource::FLiveLinkMasterLockitSource(FLiveLinkMasterLockitConnectionSettings InConnectionSettings)
	: SocketSubsystem(ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
	, ConnectionSettings(MoveTemp(InConnectionSettings))
	, LastTimeDataReceived(0.0)
	, bReceivedData(false)
{
	SourceMachineName = FText::Format(LOCTEXT("MasterLockitMachineName", "{0}:{1}"), FText::FromString(ConnectionSettings.IPAddress), FText::AsNumber(MasterLockitPortNumber, &FNumberFormattingOptions::DefaultNoGrouping()));
}

FLiveLinkMasterLockitSource::~FLiveLinkMasterLockitSource()
{
	RequestSourceShutdown();
}

void FLiveLinkMasterLockitSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;

	SubjectKey = FLiveLinkSubjectKey(InSourceGuid, ConnectionSettings.SubjectName);

	FLiveLinkStaticDataStruct MasterLockitStaticDataStruct(FLiveLinkCameraStaticData::StaticStruct());
	FLiveLinkCameraStaticData* MasterLockitStaticData = MasterLockitStaticDataStruct.Cast<FLiveLinkCameraStaticData>();

	MasterLockitStaticData->bIsFocalLengthSupported = true;
	MasterLockitStaticData->bIsApertureSupported = true;
	MasterLockitStaticData->bIsFocusDistanceSupported = true;

	MasterLockitStaticData->bIsFieldOfViewSupported = false;
	MasterLockitStaticData->bIsAspectRatioSupported = false;
	MasterLockitStaticData->bIsProjectionModeSupported = false;

	Client->PushSubjectStaticData_AnyThread(SubjectKey, ULiveLinkCameraRole::StaticClass(), MoveTemp(MasterLockitStaticDataStruct));

	OpenConnection();
}

void FLiveLinkMasterLockitSource::OnHandshakeEstablished_AnyThread()
{
	bReceivedData = true;
}

bool FLiveLinkMasterLockitSource::IsSourceStillValid() const
{
	if (Socket->GetConnectionState() != ESocketConnectionState::SCS_Connected)
	{
		return false;
	}
	else if (bReceivedData == false)
	{
		return false;
	}

	return true;
}

bool FLiveLinkMasterLockitSource::RequestSourceShutdown()
{
	if (MessageThread)
	{
		MessageThread->Stop();
		MessageThread.Reset();
	}

	if (Socket)
	{
		SocketSubsystem->DestroySocket(Socket);
		Socket = nullptr;
	}

	return true;
}

FText FLiveLinkMasterLockitSource::GetSourceType() const
{
	return LOCTEXT("MasterLockitSourceType", "MasterLockit");
}

FText FLiveLinkMasterLockitSource::GetSourceStatus() const
{
	if (Socket->GetConnectionState() == ESocketConnectionState::SCS_ConnectionError)
	{
		return LOCTEXT("FailedConnectionStatus", "Failed to connect");
	}
	else if (bReceivedData == false)
	{
		return LOCTEXT("InvalidConnectionStatus", "Connected...waiting for handshake");
	}
	else if (FPlatformTime::Seconds() - LastTimeDataReceived > DataReceivedTimeout)
	{
		return LOCTEXT("WaitingForDataStatus", "Connected...waiting for data");
	}
	return LOCTEXT("ActiveStatus", "Active");
}

void FLiveLinkMasterLockitSource::OpenConnection()
{
	check(!Socket);

	// Create an IPv4 TCP Socket
	Socket = SocketSubsystem->CreateSocket(NAME_Stream, TEXT("MasterLockit Socket"), FNetworkProtocolTypes::IPv4);
	Socket->SetNonBlocking(true);
	Socket->SetNoDelay(true);
	Socket->SetRecvErr(true);

	FIPv4Address IPAddr;
	if (FIPv4Address::Parse(ConnectionSettings.IPAddress, IPAddr) == false)
	{
		UE_LOGF(LogMasterLockitPlugin, Warning, "Ill-formed IP Address");
		return;
	}

	uint16 PortNumber = MasterLockitPortNumber;
	
	FIPv4Endpoint Endpoint = FIPv4Endpoint(IPAddr, PortNumber);
	TSharedRef<FInternetAddr> Addr = Endpoint.ToInternetAddr();
	
	if (!Socket->Connect(*Addr))
	{
		UE_LOGF(LogMasterLockitPlugin, Warning, "Could not connect the client socket to %ls:%d", *IPAddr.ToString(), PortNumber);
		return;
	}

	if (Socket->GetConnectionState() == ESocketConnectionState::SCS_ConnectionError)
	{
		UE_LOGF(LogMasterLockitPlugin, Warning, "Could not connect the client socket to %ls:%d", *IPAddr.ToString(), PortNumber);
		return;
	}

	MessageThread = MakeUnique<FMasterLockitMessageThread>(Socket);

	MessageThread->OnHandshakeEstablished_AnyThread().BindRaw(this, &FLiveLinkMasterLockitSource::OnHandshakeEstablished_AnyThread);
	MessageThread->OnFrameDataReady_AnyThread().BindRaw(this, &FLiveLinkMasterLockitSource::OnFrameDataReady_AnyThread);

	MessageThread->Start();
}

void FLiveLinkMasterLockitSource::OnFrameDataReady_AnyThread(FLensPacket InData)
{
	FLiveLinkFrameDataStruct LensFrameDataStruct(FLiveLinkCameraFrameData::StaticStruct());
	FLiveLinkCameraFrameData* LensFrameData = LensFrameDataStruct.Cast<FLiveLinkCameraFrameData>();

	LastTimeDataReceived = FPlatformTime::Seconds();
	LensFrameData->WorldTime = LastTimeDataReceived.load();
	LensFrameData->MetaData.SceneTime = InData.FrameTime;
	LensFrameData->FocusDistance = InData.FocusDistance;
	LensFrameData->FocalLength = InData.FocalLength;
	LensFrameData->Aperture = InData.Aperture;
	LensFrameData->FieldOfView = InData.HorizontalFOV;

	Client->PushSubjectFrameData_AnyThread(SubjectKey, MoveTemp(LensFrameDataStruct));
}

#undef LOCTEXT_NAMESPACE
