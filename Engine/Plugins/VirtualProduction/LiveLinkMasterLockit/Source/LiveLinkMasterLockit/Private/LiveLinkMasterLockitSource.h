// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ILiveLinkSource.h"

#include "LiveLinkMasterLockitSourceSettings.h"
#include "LiveLinkMasterLockitConnectionSettings.h"



class FMasterLockitMessageThread;
class FSocket;
class ULiveLinkSourceSettings;
struct FLensPacket;

static constexpr int MasterLockitPortNumber = 42000; // From the MasterLockit API

class ISocketSubsystem;

class LIVELINKMASTERLOCKIT_API FLiveLinkMasterLockitSource : public ILiveLinkSource
{
public:

	FLiveLinkMasterLockitSource(FLiveLinkMasterLockitConnectionSettings ConnectionSettings);
	~FLiveLinkMasterLockitSource();

	// Begin ILiveLinkSource Implementation
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;

	virtual bool IsSourceStillValid() const override;

	virtual bool RequestSourceShutdown() override;

	virtual FText GetSourceType() const override;
	virtual FText GetSourceMachineName() const override { return SourceMachineName; }
	virtual FText GetSourceStatus() const override;

	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const { return ULiveLinkMasterLockitSourceSettings::StaticClass(); }
	// End ILiveLinkSourceImplementation

private:
	void OpenConnection();
	void OnHandshakeEstablished_AnyThread();
	void OnFrameDataReady_AnyThread(FLensPacket InData);

	ILiveLinkClient* Client = nullptr;
	FSocket* Socket = nullptr;
	ISocketSubsystem* SocketSubsystem = nullptr;

	FLiveLinkMasterLockitConnectionSettings ConnectionSettings;
	FLiveLinkSubjectKey SubjectKey;
	FText SourceMachineName;

	TUniquePtr<FMasterLockitMessageThread> MessageThread;

	std::atomic<double> LastTimeDataReceived;
	std::atomic<bool> bReceivedData;

private:
	const float DataReceivedTimeout = 20.0f;
};
