// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "LiveLinkSourceSettings.h"

#include "LiveLinkMasterLockitSourceSettings.generated.h"

UCLASS()
class LIVELINKMASTERLOCKIT_API ULiveLinkMasterLockitSourceSettings : public ULiveLinkSourceSettings
{
public:
	GENERATED_BODY()
};
