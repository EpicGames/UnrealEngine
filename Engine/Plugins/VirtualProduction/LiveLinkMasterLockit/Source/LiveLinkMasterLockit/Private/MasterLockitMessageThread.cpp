// Copyright Epic Games, Inc. All Rights Reserved.

#include "MasterLockitMessageThread.h"

#include "Sockets.h"

#include "HAL/RunnableThread.h"

#include "Serialization/ArrayWriter.h"

#include "Serialization/JsonSerializer.h"

#include "Misc/SecureHash.h"

DEFINE_LOG_CATEGORY_STATIC(LogMasterLockitMessageThread, Log, All);

//~ Constant data defined in the MasterLockit API
const FString FMasterLockitProtocol::CommonCategoryString = FString(TEXT("common"));
const FString FMasterLockitProtocol::ConnectedStatusString = FString(TEXT("connected"));
const FString FMasterLockitProtocol::VendorDataCategoryString = FString(TEXT("vendorTyped"));
const FString FMasterLockitProtocol::ManufacturerNameString = FString(TEXT("manufacturerName"));
const FString FMasterLockitProtocol::UnitTypeString = FString(TEXT("unitType"));
const FString FMasterLockitProtocol::ImperialUnitString = FString(TEXT("imperial"));
const FString FMasterLockitProtocol::MetricUnitString = FString(TEXT("metric"));
const FString FMasterLockitProtocol::SerialNumberString = FString(TEXT("serialNumber"));
const FString FMasterLockitProtocol::LensTypeNameString = FString(TEXT("lensTypeName"));
const FString FMasterLockitProtocol::FirmwareVersionString = FString(TEXT("firmwareVersion"));
const FString FMasterLockitProtocol::LensTypeString = FString(TEXT("lensType"));
const FString FMasterLockitProtocol::PrimeLensString = FString(TEXT("prime"));
const FString FMasterLockitProtocol::ZoomLensString = FString(TEXT("zoom"));
const FString FMasterLockitProtocol::FrameRateString = FString(TEXT("projectFramerate"));
const FString FMasterLockitProtocol::FrameRateDelimiter = FString(TEXT("_"));
const FString FMasterLockitProtocol::DropFrameRateString = FString(TEXT("DROP"));
//~ End constant data defined in the MasterLockit API

const FString FMasterLockitDevice::ZeissLensName = FString(TEXT("Carl Zeiss AG"));

FMasterLockitMessageThread::FMasterLockitMessageThread(FSocket* InSocket) :
	Socket(InSocket)
{
}

FMasterLockitMessageThread::~FMasterLockitMessageThread()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
	}
}

void FMasterLockitMessageThread::Start()
{
	Thread.Reset(FRunnableThread::Create(this, TEXT("MasterLockit TCP Message Thread"), ThreadStackSize, TPri_AboveNormal));
}

bool FMasterLockitMessageThread::Init()
{
	bIsThreadRunning = true;
	return true;
}

void FMasterLockitMessageThread::Stop()
{
	bIsThreadRunning = false;
}

uint32 FMasterLockitMessageThread::Run() 
{
	// Pre-allocate a buffer to receive data into from the TCP socket
	FArrayReader ReceiveBuffer;
	ReceiveBuffer.SetNumZeroed(ReceiveBufferSize);

	Packet.SetNumZeroed(ReceiveBufferSize);

	uint32 ReceiveBufferTail = ReceiveBuffer.Tell();
	int32 NumBytesReceived = 0;

	// Generate a map from MasterLockit frame rate strings to FFrameRates to simplify parsing
	GenerateFrameRateMap();

	while (bIsThreadRunning)
	{
		if (Socket->Recv(ReceiveBuffer.GetData() + ReceiveBufferTail, ReceiveBuffer.Num() - ReceiveBufferTail, NumBytesReceived))
		{
			ReceiveBufferTail += NumBytesReceived;

			// Attempt to break up packets from the server until there are no full packets remaining in the buffer
			while (true)
			{
				// The first 4 bytes of a packet contain the length of the remainder of the packet
				// Ensure that there are at least 4 bytes in the buffer to read
				if (ReceiveBufferTail < sizeof(uint32))
				{
					break;
				}

				// Read the length of the next packet in the buffer
				uint32 PacketLength;
				ReceiveBuffer << PacketLength;		
				PacketLength += sizeof(uint32); // The packet includes the 4 bytes of length data

				ReceiveBuffer.Seek(0);

				// Check if the buffer received enough data for the full packet
				if (PacketLength > ReceiveBufferTail)
				{
					break;
				}

				// Copy one full packet from the buffer
				ReceiveBuffer.Serialize(Packet.GetData(), PacketLength);

				// Remove the packet from the buffer
				ReceiveBuffer.RemoveAt(0, PacketLength, EAllowShrinking::No);
				ReceiveBuffer.AddZeroed(PacketLength);
				ReceiveBuffer.Seek(0);

				ReceiveBufferTail -= PacketLength;

				ParsePacket(Packet);
			}
		}
		else if (Socket->GetConnectionState() == ESocketConnectionState::SCS_ConnectionError)
		{
			break;
		}
	}

	return 0;
}

void FMasterLockitMessageThread::ParsePacket(FArrayReader& InPacketData)
{
	InPacketData.Seek(0);
	uint32 PacketSize = InPacketData.TotalSize();
	check(PacketSize > 0);

	uint32 MessageSize;
	InPacketData << MessageSize;
	MessageSize += sizeof(uint32);

	EMasterLockitMessageType MessageType;
	InPacketData << MessageType;

	switch (MessageType)
	{
	case EMasterLockitMessageType::ServerToClientHandshake:
	{
		EServerHandshakeMessageCode MessageCode;
		InPacketData << MessageCode;

		switch (MessageCode)
		{
		case EServerHandshakeMessageCode::Challenge:
		{
			InitiateHandshake();
			break;
		}
		case EServerHandshakeMessageCode::Success:
		{
			HandshakeEstablishedDelegate.ExecuteIfBound();

			SubscribeToDeviceMetadataUpdates(EMasterLockitDeviceType::TimecodeGenerator);
			SubscribeToDeviceMetadataUpdates(EMasterLockitDeviceType::Lens);

			break;
		}
		case EServerHandshakeMessageCode::Failure:
		{
			break;
		}
		default:
		{
			break;
		}
		}

		break;
	}
	case EMasterLockitMessageType::ServerToClientDataRequest:
	{
		EServerDataRequestMessageCode MessageCode;
		InPacketData << MessageCode;

		switch (MessageCode)
		{
		case EServerDataRequestMessageCode::DataEvent:
		{
			InPacketData.Seek(0);
			AcknowledgeMessageFromServer(InPacketData, MessageSize);
			break;
		}
		case EServerDataRequestMessageCode::Ping:
		{
			InPacketData.Seek(0);
			AcknowledgeMessageFromServer(InPacketData, MessageSize);
			break;
		}
		default: // Error
		{
			break;
		}
		}

		break;
	}
	case EMasterLockitMessageType::ServerToClientNotification:
	{
		EServerNotificationMessageCode MessageCode;
		InPacketData << MessageCode;

		uint64 DeviceID;
		InPacketData << DeviceID;

		EMasterLockitDeviceType DeviceType;
		InPacketData << DeviceType;

		switch (MessageCode)
		{
		case EServerNotificationMessageCode::DeviceAppeared:
		{
			if (!DetectedDevices.Contains(DeviceID))
			{
				FMasterLockitDevice NewDevice;
				NewDevice.DeviceType = DeviceType;
				DetectedDevices.Add(DeviceID, NewDevice);
			}

			DetectedDevices[DeviceID].bHasAppeared = true;

			SubscribeToVolatileDataUpdates(DeviceID, DeviceType);

			break;
		}
		case EServerNotificationMessageCode::DeviceDisappeared:
		{
			// Unimplemented
			break;
		}
		case EServerNotificationMessageCode::DeviceMetaEvent:
		{
			uint32 PayloadSize = MessageSize - InPacketData.Tell();
			TArray<ANSICHAR> Payload;
			Payload.SetNumUninitialized(PayloadSize);
			InPacketData.Serialize(Payload.GetData(), PayloadSize);

			if (!DetectedDevices.Contains(DeviceID))
			{
				FMasterLockitDevice NewDevice;
				NewDevice.DeviceType = DeviceType;
				DetectedDevices.Add(DeviceID, NewDevice);
			}

			FMasterLockitDevice& Device = DetectedDevices[DeviceID];

			// The payload data for this packet is formatted as json
			TSharedPtr<FJsonObject> JsonObject = MakeShared<FJsonObject>();
			FString JsonString = Payload.GetData();
			JsonString.RemoveAt(Payload.Num(), JsonString.Len() + 1 - Payload.Num());
			TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(JsonString);
			FJsonSerializer::Deserialize(JsonReader, JsonObject);

			// Acquire references to metadata categories that are present in all DeviceTypes 
			if (!JsonObject->HasField(FMasterLockitProtocol::CommonCategoryString) ||
				!JsonObject->HasField(FMasterLockitProtocol::VendorDataCategoryString))
			{
				break;
			}

			const TSharedPtr<FJsonObject>& commonData = JsonObject->GetFieldUntyped(FMasterLockitProtocol::CommonCategoryString)->AsObject();
			const TSharedPtr<FJsonObject>& vendorData = JsonObject->GetFieldUntyped(FMasterLockitProtocol::VendorDataCategoryString)->AsObject();

			switch (DeviceType)
			{
			case EMasterLockitDeviceType::Lens:
			{
				if (Device.bHasAppeared)
				{
					SubscribeToVolatileDataUpdates(DeviceID, DeviceType);
				}

				vendorData->TryGetStringField(FMasterLockitProtocol::ManufacturerNameString, Device.ManufacturerName);
				vendorData->TryGetStringField(FMasterLockitProtocol::SerialNumberString, Device.SerialNumber);
				vendorData->TryGetStringField(FMasterLockitProtocol::LensTypeNameString, Device.LensName);
				
				FString FirmwareVersionString;
				if (vendorData->TryGetStringField(FMasterLockitProtocol::FirmwareVersionString, FirmwareVersionString))
				{
					TArray<FString> Tokens;
					FirmwareVersionString.ParseIntoArray(Tokens, TEXT("."), true);
					if (Tokens.Num() == 2)
					{
						Device.FirmwareMajorVersion = FCString::Atoi(*Tokens[0]);
						Device.FirmwareMinorVersion = FCString::Atoi(*Tokens[1]);
					}
				}

				FString UnitTypeString;
				if (vendorData->TryGetStringField(FMasterLockitProtocol::UnitTypeString, UnitTypeString))
				{
					if (UnitTypeString == FMasterLockitProtocol::ImperialUnitString)
					{
						Device.UnitType = EMasterLockitUnitType::Imperial;
					}
					else
					{
						Device.UnitType = EMasterLockitUnitType::Metric;
					}
				}

				FString LensTypeString;
				if (vendorData->TryGetStringField(FMasterLockitProtocol::LensTypeString, LensTypeString))
				{
					if (LensTypeString == FMasterLockitProtocol::PrimeLensString)
					{
						Device.LensType = EMasterLockitLensType::Prime;
					}
					else
					{
						Device.LensType = EMasterLockitLensType::Zoom;
					}
				}

				break;
			}
			case EMasterLockitDeviceType::TimecodeGenerator:
			{
				bool bIsConnected;
				if (commonData->TryGetBoolField(FMasterLockitProtocol::ConnectedStatusString, bIsConnected))
				{
					if (!bIsConnected)
					{
						break;
					}

					FString FrameRateString;
					if (vendorData->TryGetStringField(FMasterLockitProtocol::FrameRateString, FrameRateString))
					{
						if (FrameRates.Contains(FrameRateString))
						{
							MasterLockitFrameRate = FrameRates[FrameRateString];
						}

						// Parse the frame rate string to determine if it is a drop frame rate
						TArray<FString> Tokens;
						FrameRateString.ParseIntoArray(Tokens, *FMasterLockitProtocol::FrameRateDelimiter, true);
						if (Tokens.Last() == FMasterLockitProtocol::DropFrameRateString)
						{
							bIsDropFrameRate = true;
						}
						else
						{
							bIsDropFrameRate = false;
						}
					}
				}

				break;
			}
			}

			break;
		}
		case EServerNotificationMessageCode::VolatileDataEvent:
		{
			switch (DeviceType)
			{
			case EMasterLockitDeviceType::Camera:
			{
				// Unimplemented
				break;
			}
			case EMasterLockitDeviceType::SoundRecorder:
			{
				// Unimplemented
				break;
			}
			case EMasterLockitDeviceType::TimecodeGenerator:
			{
				// Unimplemented
				break;
			}
			case EMasterLockitDeviceType::SlateInfoDevice:
			{
				// Unimplemented
				break;
			}
			case EMasterLockitDeviceType::Slate:
			{
				// Unimplemented
				break;
			}
			case EMasterLockitDeviceType::Storage:
			{
				// Unimplemented
				break;
			}
			case EMasterLockitDeviceType::Lens:
			{
				if (DetectedDevices.Contains(DeviceID))
				{
					FMasterLockitDevice Device = DetectedDevices[DeviceID];
					if (Device.ManufacturerName.Equals(FMasterLockitDevice::ZeissLensName))
					{
						ParseZeissLensData(InPacketData, Device);
					}
				}

				break;
			}
			default: // Error
			{
				break;
			}
			}

			break;
		}
		default: // Error
		{
		}
		}
		break;
	}
	case EMasterLockitMessageType::ServerToClientAcknowledge:
	{
		// Unimplemented
		break;
	}
	case EMasterLockitMessageType::ServerToClientNegativeAcknowledge:
	{
		// Unimplemented
		break;
	}
	default: // Error, unknown message type, likely an ill-formed packet
	{
		break;
	}
	}
}

void FMasterLockitMessageThread::InitiateHandshake()
{
	FArrayWriter ReplyMessage;

	uint32 ZeroMessageLength = 0;

	const EMasterLockitMessageType MessageType = EMasterLockitMessageType::ClientToServerHandshake;

	int64 TimeStamp = FDateTime::UtcNow().ToUnixTimestamp();

	uint64 DeviceID = 0xBABABABABABABABA;

	FString Alias = FString(TEXT("Unreal Engine 4"));

	const TArray<uint8> ProtocolVersions = { 0x01 }; // Specified by the MasterLockitAPI
	int32 NumProtocolVersions = ProtocolVersions.Num();

	constexpr uint32 APIKeyLength = 24; // Specified by the MasterLockit API
	TArray<uint8> APIKey;
	APIKey.SetNumUninitialized(APIKeyLength); // The MasterLockit API does not validate this input

	constexpr uint32 SignatureLength = 32; // Specified by the MasterLockit API
	TArray<uint8> Signature;
	Signature.SetNumUninitialized(SignatureLength); // The MasterLockit API does not validate this input

	ReplyMessage << ZeroMessageLength;
	ReplyMessage << MessageType;
	ReplyMessage << TimeStamp;
	ReplyMessage << DeviceID;
	ReplyMessage << Alias;
	ReplyMessage << NumProtocolVersions;

	for (uint8 Version : ProtocolVersions)
	{
		ReplyMessage << Version;
	}

	ReplyMessage.Serialize(APIKey.GetData(), APIKey.Num());
	ReplyMessage.Serialize(Signature.GetData(), Signature.Num());

	uint32 MessageLength = ReplyMessage.Num() - sizeof(ZeroMessageLength);
	ReplyMessage.Seek(0);
	ReplyMessage << MessageLength;

	SendMessageToServer(ReplyMessage);
}

void FMasterLockitMessageThread::AcknowledgeMessageFromServer(const TArray<uint8> InMessageFromServer, const uint32 InServerMessageLength)
{
	// Client-to-Server Acknowledge Message Format (from MasterLockit API)
	// -------------------------------------------------------------
	// Message Length	Message Type	SHA-1 of Message From Server
	// 4 bytes			1 byte			20 bytes 
	// -------------------------------------------------------------

	// Construct the individual data elements of the message
	uint32 ZeroMessageLength = 0; // 4-byte placeholder to be overwritten when actual message length is known

	EMasterLockitMessageType MessageType = EMasterLockitMessageType::ClientToServerAcknowledge;

	constexpr uint32 HashLength = 20; // See comment on FSHA1::HashBuffer(...)
	TArray<uint8> MessageFromServerSHA1;
	MessageFromServerSHA1.SetNumUninitialized(HashLength);
	FSHA1::HashBuffer(InMessageFromServer.GetData(), InServerMessageLength, MessageFromServerSHA1.GetData());

	// Write each data element of the message to an archive
	FArrayWriter Message;
	Message << ZeroMessageLength;
	Message << MessageType;
	Message.Serialize(MessageFromServerSHA1.GetData(), MessageFromServerSHA1.Num());

	// Compute the message length (excluding the 4 bytes for the message length data itself)
	uint32 MessageLength = Message.Num() - sizeof(ZeroMessageLength);

	// Overwrite the zero message length with valid data
	Message.Seek(0);
	Message << MessageLength;

	// Send the archive data to the server over the connected socket
	SendMessageToServer(Message);
}

void FMasterLockitMessageThread::SendMessageToServer(const FArrayWriter InMessageToSend)
{
	int32 Sent;
	Socket->Send(InMessageToSend.GetData(), InMessageToSend.Num(), Sent);

	if (Sent != InMessageToSend.Num())
	{
		UE_LOGF(LogMasterLockitMessageThread, Warning, "Full message was not sent to the MasterLockit server");
	}
}

void FMasterLockitMessageThread::SubscribeToDeviceMetadataUpdates(const EMasterLockitDeviceType InDeviceType)
{
	// Client-to-Server Subscribe to Device Metadata Updates Message Format (from MasterLockit API)
	// ------------------------------------------------------------
	// Message Length	Message Type	Message Code	Device Type
	// 4 bytes			1 byte			1 byte			1 byte
	// ------------------------------------------------------------

	// Construct the individual data elements of the message
	uint32 ZeroMessageLength = 0;
	EMasterLockitMessageType MessageType = EMasterLockitMessageType::ClientToServerDataRequest;
	EClientDataRequestMessageCode MessageCode = EClientDataRequestMessageCode::SubscribeDeviceUpdates;

	// Write each data element of the message to an archive
	FArrayWriter Message;
	Message << ZeroMessageLength;
	Message << MessageType;
	Message << MessageCode;
	Message << InDeviceType;

	uint32 MessageLength = Message.Num() - sizeof(ZeroMessageLength);
	Message.Seek(0);
	Message << MessageLength;

	SendMessageToServer(Message);

	HashDataRequestMessage(Message, "Subscribe to Device Metadata Events");
}

void FMasterLockitMessageThread::SubscribeToVolatileDataUpdates(uint64 InDeviceID, const EMasterLockitDeviceType InDeviceType)
{
	// Client-to-Server Subscribe to Volatile Data Updates Message Format (from MasterLockit API)
	// ------------------------------------------------------------------------
	// Message Length	Message Type	Message Code	Device ID	Device Type
	// 4 bytes			1 byte			1 byte			8 bytes		1 byte
	// ------------------------------------------------------------------------

	// Construct the individual data elements of the message
	uint32 ZeroMessageLength = 0;
	EMasterLockitMessageType MessageType = EMasterLockitMessageType::ClientToServerDataRequest;
	EClientDataRequestMessageCode MessageCode = EClientDataRequestMessageCode::SubscribeVolatileDataUpdates;

	// Write each data element of the message to an archive
	FArrayWriter Message;
	Message << ZeroMessageLength;
	Message << MessageType;
	Message << MessageCode;
	Message << InDeviceID;
	Message << InDeviceType;

	uint32 MessageLength = Message.Num() - sizeof(ZeroMessageLength);
	Message.Seek(0);
	Message << MessageLength;

	SendMessageToServer(Message);

	HashDataRequestMessage(Message, "Subscribe to Volatile Data Events");
}

void FMasterLockitMessageThread::HashDataRequestMessage(const FArrayWriter InMessage, const FString InRequestName)
{
	constexpr uint32 HashLength = 20; // See comment on FSHA1::HashBuffer(...)
	FArrayReader MessageHash;
	MessageHash.SetNumUninitialized(HashLength);
	FSHA1::HashBuffer(InMessage.GetData(), InMessage.Num(), MessageHash.GetData());

	DataRequests.Add(MessageHash, InRequestName);
}

void FMasterLockitMessageThread::ParseZeissLensData(FArrayReader& InData, const FMasterLockitDevice& Device)
{
	FLensPacket LensData;

	// Zeiss Specific Lens Data Format
	uint32 TimeCodeSeconds;
	uint16 TimeCodeFrames;
	uint8 Tag;

	InData << TimeCodeSeconds << TimeCodeFrames << Tag;

	InData.Serialize(ZeissRawPacket.RawFocusDistance.GetData(), ZeissRawPacket.RawFocusDistance.Num());
	InData.Serialize(ZeissRawPacket.RawAperture.GetData(), ZeissRawPacket.RawAperture.Num());
	InData.Serialize(ZeissRawPacket.RawHorizontalFOV.GetData(), ZeissRawPacket.RawHorizontalFOV.Num());
	InData.Serialize(ZeissRawPacket.RawEntrancePupilPosition.GetData(), ZeissRawPacket.RawEntrancePupilPosition.Num());
	InData.Serialize(ZeissRawPacket.RawShadingData.GetData(), ZeissRawPacket.RawShadingData.Num());
	InData.Serialize(ZeissRawPacket.RawDistortionData.GetData(), ZeissRawPacket.RawDistortionData.Num());

	// The packet from the Zeiss lens must begin with a 'z' character tag
	if (Tag != 'z')
	{
		return;
	}

	// Parse Packed Focus Distance Data
	// Current focus distance units (1mm or 1/10 inch) is dependent on the current display units of the lens
	ZeissRawPacket.RawFocusDistance[0] = ZeissRawPacket.RawFocusDistance[0] & 0x3F;
	ZeissRawPacket.RawFocusDistance[1] = ZeissRawPacket.RawFocusDistance[1] & 0x3F;
	ZeissRawPacket.RawFocusDistance[2] = ZeissRawPacket.RawFocusDistance[2] & 0x3F;
	ZeissRawPacket.RawFocusDistance[3] = ZeissRawPacket.RawFocusDistance[3] & 0x3F;

	LensData.FocusDistance = (ZeissRawPacket.RawFocusDistance[0] << 18) + (ZeissRawPacket.RawFocusDistance[1] << 12) + (ZeissRawPacket.RawFocusDistance[2] << 6) + (ZeissRawPacket.RawFocusDistance[3]);

	// Depending on the unit type of the device, the focus distance will either be in mm or 1/10 inches
	if (Device.UnitType == EMasterLockitUnitType::Imperial)
	{
		LensData.FocusDistance = LensData.FocusDistance / 10.0f; // Convert to inches
		LensData.FocusDistance = LensData.FocusDistance * 2.54; // Convert to cm
	}
	else
	{
		LensData.FocusDistance = LensData.FocusDistance / 10.0f; // Convert to cm
	}

	// Parse Packed Aperture Data
	LensData.Aperture = (((ZeissRawPacket.RawAperture[1] & 0x40) << 7) + (ZeissRawPacket.RawAperture[0] & 0x7F)) / 10.0f;
	uint8 ApertureFraction = ZeissRawPacket.RawAperture[1] & 0x0F;

	LensData.Aperture += (ApertureFraction / 10.0f);

	// Parse Horizontal FOV
	LensData.HorizontalFOV = ((ZeissRawPacket.RawHorizontalFOV[0] & 0x1F) << 6) + (ZeissRawPacket.RawHorizontalFOV[1] & 0x3F);
	LensData.HorizontalFOV /= 10.0f;

	// Parse Entrance Pupil Position
	bool bIsNegative = (((ZeissRawPacket.RawEntrancePupilPosition[0] & 0x20) >> 5) == 1);
	LensData.EntrancePupilPosition = ((ZeissRawPacket.RawEntrancePupilPosition[0] & 0x0F) << 6) + (ZeissRawPacket.RawEntrancePupilPosition[1] & 0x3F);
	if (bIsNegative)
	{
		LensData.EntrancePupilPosition *= -1;
	}

	// Parse Shading Data
	LensData.ShadingData[0] = ((ZeissRawPacket.RawShadingData[0] << 8) + ZeissRawPacket.RawShadingData[1]) / 10.0f;
	LensData.ShadingData[1] = ((ZeissRawPacket.RawShadingData[2] << 8) + ZeissRawPacket.RawShadingData[3]) / 10.0f;
	LensData.ShadingData[2] = ((ZeissRawPacket.RawShadingData[4] << 8) + ZeissRawPacket.RawShadingData[5]) / 10.0f;
	LensData.ShadingData[3] = ((ZeissRawPacket.RawShadingData[6] << 8) + ZeissRawPacket.RawShadingData[7]) / 10.0f;
	LensData.ShadingData[4] = ((ZeissRawPacket.RawShadingData[8] << 8) + ZeissRawPacket.RawShadingData[9]) / 10.0f;
	LensData.ShadingData[5] = ((ZeissRawPacket.RawShadingData[10] << 8) + ZeissRawPacket.RawShadingData[11]) / 10.0f;

	// Parse Distortion Data
	LensData.DistortionData[0] = int16((ZeissRawPacket.RawDistortionData[0] << 8) + ZeissRawPacket.RawDistortionData[1]) / 10.0f;
	LensData.DistortionData[1] = int16((ZeissRawPacket.RawDistortionData[2] << 8) + ZeissRawPacket.RawDistortionData[3]) / 10.0f;
	LensData.DistortionData[2] = int16((ZeissRawPacket.RawDistortionData[4] << 8) + ZeissRawPacket.RawDistortionData[5]) / 10.0f;
	LensData.DistortionData[3] = int16((ZeissRawPacket.RawDistortionData[6] << 8) + ZeissRawPacket.RawDistortionData[7]) / 10.0f;
	LensData.DistortionData[4] = int16((ZeissRawPacket.RawDistortionData[8] << 8) + ZeissRawPacket.RawDistortionData[9]) / 10.0f;
	LensData.DistortionData[5] = int16((ZeissRawPacket.RawDistortionData[10] << 8) + ZeissRawPacket.RawDistortionData[11]) / 10.0f;

	// A prime lens has a fixed focal length, so the min and max focal length are the same
	if (((Device.FirmwareMajorVersion == 1) && (Device.FirmwareMinorVersion >= 71)) || (Device.FirmwareMajorVersion > 1))
	{
		constexpr float SensorSize = 24.9f; // The width of a Super35 sensor, which is 24.9mm
		LensData.FocalLength = (SensorSize / 2.f) / FMath::Tan(FMath::DegreesToRadians(LensData.HorizontalFOV / 2.f));
	}
	else
	{
		const float SensorSize = FMath::Sqrt((36.0f * 36.0f) + (24.0f * 24.0f)); // The diagonal of a Full Frame sensor, which is 36mm x 24mm
		LensData.FocalLength = (SensorSize / 2.f) / FMath::Tan(FMath::DegreesToRadians(LensData.HorizontalFOV / 2.f));
	}

	FTimecode Timecode = FTimecode(TimeCodeSeconds + TimeCodeFrames / MasterLockitFrameRate.AsDecimal(), MasterLockitFrameRate, bIsDropFrameRate, false);
	LensData.FrameTime = FQualifiedFrameTime(Timecode, MasterLockitFrameRate);

	FrameDataReadyDelegate.ExecuteIfBound(LensData);
}

void FMasterLockitMessageThread::GenerateFrameRateMap()
{
	FrameRates.Add("FPS_23_976", FFrameRate(24000, 1001));
	FrameRates.Add("FPS_24", FFrameRate(24, 1));
	FrameRates.Add("FPS_25", FFrameRate(25, 1));
	FrameRates.Add("FPS_29_97", FFrameRate(30000, 1001));
	FrameRates.Add("FPS_29_97_DROP", FFrameRate(30000, 1001));
	FrameRates.Add("FPS_30", FFrameRate(30, 1));
	FrameRates.Add("FPS_47_952", FFrameRate(48000, 1001));
	FrameRates.Add("FPS_48", FFrameRate(48, 1));
	FrameRates.Add("FPS_50", FFrameRate(50, 1));
	FrameRates.Add("FPS_59_94", FFrameRate(60000, 1001));
	FrameRates.Add("FPS_59_94_DROP", FFrameRate(60000, 1001));
	FrameRates.Add("FPS_60", FFrameRate(60, 1));
}
