// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Runnable.h"

#include "Misc/QualifiedFrameTime.h"

#include "Serialization/ArrayReader.h"

class FRunnable;
class FSocket;

class FArrayReader;
class FBufferArchive;
typedef FBufferArchive FArrayWriter;

struct FLensPacket;

DECLARE_DELEGATE_OneParam(FOnFrameDataReady, FLensPacket InData);
DECLARE_DELEGATE(FOnHandshakeEstablished);

struct FLensPacket
{
	FQualifiedFrameTime FrameTime;

	float FocusDistance = 0.0f;
	float Aperture = 0.0f;
	float HorizontalFOV = 0.0f;
	int16 EntrancePupilPosition = 0;
	float ShadingData[6] = { 0.0f };
	float DistortionData[6] = { 0.0f };

	float FocalLength = 0.0f;
};

struct FMessageHash
{
	TArray<uint8> Hash;

	FMessageHash(TArray<uint8> InArray) { Hash = InArray; }

	bool operator==(const FMessageHash& OtherMessage) const
	{
		return (Hash == OtherMessage.Hash);
	}
};

FORCEINLINE uint32 GetTypeHash(const FMessageHash& MessageHash)
{
	return FCrc::MemCrc32(MessageHash.Hash.GetData(), MessageHash.Hash.Num());
}

enum class EMasterLockitMessageType : uint8
{
	ClientToServerDataRequest = 0x01,
	ClientToServerNotification = 0x02,
	ClientToServerAcknowledge = 0x03,
	ClientToServerNegativeAcknowledge = 0x04,
	ClientToServerHandshake = 0x0F,

	ServerToClientDataRequest = 0x11,
	ServerToClientNotification = 0x12,
	ServerToClientAcknowledge = 0x13,
	ServerToClientNegativeAcknowledge = 0x14,
	ServerToClientHandshake = 0x1F
};

enum class EServerHandshakeMessageCode : uint8
{
	Challenge = 0x00,
	Success = 0x01,
	Failure = 0x02
};

enum class EServerDataRequestMessageCode : uint8
{
	DataEvent = 0x01,
	Ping = 0x02
};

enum class EServerNotificationMessageCode : uint8
{
	DeviceAppeared = 0x01,
	DeviceDisappeared = 0x02,
	DeviceMetaEvent = 0x04,
	VolatileDataEvent = 0x05
};

enum class EClientDataRequestMessageCode : uint8
{
	SubscribeDataUpdates = 0x01,
	UnsubscribeDataUpdates = 0x02,
	SubscribeDeviceUpdates = 0x03,
	UnsubscribeDeviceUpdates = 0x04,
	REGISTER_AS_DEVICE = 0x05,
	UnregisterAsDevice = 0x06,
	DataEvent = 0x07,
	FetchClips = 0x08,
	DeleteClips = 0x09,
	Export = 0x0A,
	SubscribeVolatileDataUpdates = 0x0B,
	UnsubscribeVolatileDataUpdates = 0x0C,
	FetechLensData = 0x0D,
	StartCJam = 0x0E,
	RenameDevice = 0x0F,
	RemoveDevice = 0x10
};

enum class EClientNotificationMessageCode : uint8
{
	DeviceMetaEvent = 0x01
};

enum class EMasterLockitDeviceType : uint8
{
	Unknown = 0x00,
	Camera = 0x01,
	SoundRecorder = 0x02,
	TimecodeGenerator = 0x03,
	SlateInfoDevice = 0x04,
	Slate = 0x05,
	Storage = 0x06,
	Lens = 0x07,
};

enum class EMasterLockitUnitType : uint8
{
	Unknown = 0x00,
	Imperial = 0x01,
	Metric = 0x02,
};

enum class EMasterLockitLensType : uint8
{
	Unknown = 0x00,
	Prime = 0x01,
	Zoom = 0x02,
};

struct FMasterLockitDevice
{
	uint64 DeviceID = 0x00;
	EMasterLockitDeviceType DeviceType = EMasterLockitDeviceType::Unknown;

	// Whether the MasterLockit has sent a DeviceAppeared message about this device yet
	bool bHasAppeared = false;

	//~ Begin Zeiss Lens Specific Vendor Data
	static const FString ZeissLensName;

	FString ManufacturerName;
	FString LensName;
	FString SerialNumber;

	uint16 FirmwareMajorVersion = 0;
	uint16 FirmwareMinorVersion = 0;

	EMasterLockitUnitType UnitType = EMasterLockitUnitType::Unknown;
	EMasterLockitLensType LensType = EMasterLockitLensType::Unknown;
	//~ End Zeiss Lens Specific Vendor Data
};

struct FZeissRawPacket
{
	static const uint32 NumFocusDistanceBytes = 4;
	static const uint32 NumApertureBytes = 2;
	static const uint32 NumHorizontalFOVBytes = 2;
	static const uint32 NumEntrancePupilBytes = 2;
	static const uint32 NumShadingBytes = 12;
	static const uint32 NumDistortionBytes = 12;

	TArray<uint8> RawFocusDistance;
	TArray<uint8> RawAperture;
	TArray<uint8> RawHorizontalFOV;
	TArray<uint8> RawEntrancePupilPosition;
	TArray<uint8> RawShadingData;
	TArray<uint8> RawDistortionData;

	FZeissRawPacket() 
	{
		RawFocusDistance.SetNumUninitialized(NumFocusDistanceBytes);
		RawAperture.SetNumUninitialized(NumApertureBytes);
		RawHorizontalFOV.SetNumUninitialized(NumHorizontalFOVBytes);
		RawEntrancePupilPosition.SetNumUninitialized(NumEntrancePupilBytes);
		RawShadingData.SetNumUninitialized(NumShadingBytes);
		RawDistortionData.SetNumUninitialized(NumDistortionBytes);
	}
};

struct FMasterLockitProtocol
{
	static const FString CommonCategoryString;
	static const FString ConnectedStatusString;

	static const FString VendorDataCategoryString;
	static const FString ManufacturerNameString;

	static const FString UnitTypeString;
	static const FString ImperialUnitString;
	static const FString MetricUnitString;

	static const FString SerialNumberString;
	static const FString LensTypeNameString;
	static const FString FirmwareVersionString;

	static const FString LensTypeString;
	static const FString PrimeLensString;
	static const FString ZoomLensString;

	static const FString FrameRateString;
	static const FString FrameRateDelimiter;
	static const FString DropFrameRateString;
};

class FMasterLockitMessageThread : public FRunnable
{
public:

	FMasterLockitMessageThread(FSocket* InSocket);
	~FMasterLockitMessageThread();

	void Start();

	/**
	 * Returns a delegate that is executed when frame data is ready.
	 *
	 * @return The delegate.
	 */
	FOnFrameDataReady& OnFrameDataReady_AnyThread()
	{
		return FrameDataReadyDelegate;
	}

	FOnHandshakeEstablished& OnHandshakeEstablished_AnyThread()
	{
		return HandshakeEstablishedDelegate;
	}

public:

	//~ FRunnable Interface
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	// End FRunnable Interface

private:

	void GenerateFrameRateMap();

	void ParsePacket(FArrayReader& InData);

	void InitiateHandshake();

	void SendMessageToServer(const FArrayWriter InMessageToSend);
	void AcknowledgeMessageFromServer(const TArray<uint8> InMessageFromServer, const uint32 InServerMessageLength);

	void HashDataRequestMessage(const FArrayWriter InMessage, const FString InRequestName);

	void SubscribeToDeviceMetadataUpdates(const EMasterLockitDeviceType InDeviceType);
	void SubscribeToVolatileDataUpdates(uint64 InDeviceID, const EMasterLockitDeviceType InDeviceType);

	void ParseZeissLensData(FArrayReader& InData, const FMasterLockitDevice& Device);

private:

	FSocket* const Socket;
	TUniquePtr<FRunnableThread> Thread;
	bool bIsThreadRunning = false;

	TMap<FMessageHash, FString> DataRequests;
	TMap<uint64, FMasterLockitDevice> DetectedDevices;

	TMap<FString, FFrameRate> FrameRates;
	FFrameRate MasterLockitFrameRate = { 24, 0 };
	bool bIsDropFrameRate = false;

	FOnHandshakeEstablished HandshakeEstablishedDelegate;
	FOnFrameDataReady FrameDataReadyDelegate;

	// Pre-allocated space to copy the data for one MasterLockit packet into
	FArrayReader Packet;

	// Pre-allocated space to read raw (unparsed) Zeiss lens data into
	FZeissRawPacket ZeissRawPacket;

private:
	// In practice, 32KB is more than enough space to hold as much data as the socket can send at one time
	static constexpr uint32 ReceiveBufferSize = 1024 * 32;
	
	static constexpr uint32 ThreadStackSize = 1024 * 128;

};
