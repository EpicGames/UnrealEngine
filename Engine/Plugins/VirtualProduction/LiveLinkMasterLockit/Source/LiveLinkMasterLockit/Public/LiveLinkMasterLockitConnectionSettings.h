// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkMasterLockitConnectionSettings.generated.h"

USTRUCT()
struct FLiveLinkMasterLockitConnectionSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Settings")
	FString IPAddress = TEXT("0.0.0.0");

	UPROPERTY(EditAnywhere, Category = "Settings")
	FName SubjectName = TEXT("MasterLockitDevice");
};
