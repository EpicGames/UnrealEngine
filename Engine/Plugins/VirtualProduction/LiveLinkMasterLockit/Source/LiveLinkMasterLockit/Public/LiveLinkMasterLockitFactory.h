// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceFactory.h"

#include "LiveLinkMasterLockitFactory.generated.h"

class SLiveLinkMasterLockitSourcePanel;
struct FLiveLinkMasterLockitConnectionSettings;

UCLASS()
class LIVELINKMASTERLOCKIT_API ULiveLinkMasterLockitSourceFactory : public ULiveLinkSourceFactory
{
public:
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedPtr<SWidget>, FBuildCreationPanelDelegate, const ULiveLinkMasterLockitSourceFactory*, FOnLiveLinkSourceCreated);
	static FBuildCreationPanelDelegate OnBuildCreationPanel;

public:
	GENERATED_BODY()

	//~ Begin ULiveLinkSourceFactory interface
	virtual FText GetSourceDisplayName() const;
	virtual FText GetSourceTooltip() const;

	virtual EMenuType GetMenuType() const override { return EMenuType::SubPanel; }
	virtual TSharedPtr<SWidget> BuildCreationPanel(FOnLiveLinkSourceCreated OnLiveLinkSourceCreated) const override;
	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& ConnectionString) const override;
	//~ End ULiveLinkSourceFactory interface

	static FString CreateConnectionString(const FLiveLinkMasterLockitConnectionSettings& Settings);
};
