// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMasterLockitEditorModule.h"

#include "LiveLinkMasterLockitFactory.h"
#include "LiveLinkMasterLockitSourcePanel.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "LiveLinkMasterLockitEditorModule"

void FLiveLinkMasterLockitEditorModule::StartupModule()
{
	// register the LiveLinkFactory panel
	auto BuildCreationPanel = [](const ULiveLinkMasterLockitSourceFactory* Factory, ULiveLinkSourceFactory::FOnLiveLinkSourceCreated OnSourceCreated)
		-> TSharedPtr<SWidget>
	{
		return SNew(SLiveLinkMasterLockitSourcePanel)
			.Factory(Factory)
			.OnSourceCreated(OnSourceCreated);
	};
	ULiveLinkMasterLockitSourceFactory::OnBuildCreationPanel.BindLambda(BuildCreationPanel);
}

void FLiveLinkMasterLockitEditorModule::ShutdownModule()
{
	ULiveLinkMasterLockitSourceFactory::OnBuildCreationPanel.Unbind();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FLiveLinkMasterLockitEditorModule, LiveLinkMasterLockitEditor)
