// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkMasterLockitSourcePanel.h"

#include "DetailsViewArgs.h"
#include "IStructureDetailsView.h"

#include "LiveLinkMasterLockitFactory.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#include "UObject/StructOnScope.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "LiveLinkMasterLockitSourcePanel"

void SLiveLinkMasterLockitSourcePanel::Construct(const FArguments& Args)
{
	SourceFactory = Args._Factory;
	OnSourceCreated = Args._OnSourceCreated;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowPropertyMatrixButton = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.ViewIdentifier = NAME_None;
	DetailsViewArgs.bShowCustomFilterOption = false;
	DetailsViewArgs.bShowOptions = false;

	FStructureDetailsViewArgs StructViewArgs;

	TSharedRef<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(FLiveLinkMasterLockitConnectionSettings::StaticStruct(), reinterpret_cast<uint8*>(&ConnectionSettings));
	TSharedRef<IStructureDetailsView> StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, StructViewArgs, StructOnScope);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			StructureDetailsView->GetWidget().ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(EHorizontalAlignment::HAlign_Right)
			[
				SNew(SButton)
				.Text(LOCTEXT("OkButton", "Ok"))
				.OnClicked(this, &SLiveLinkMasterLockitSourcePanel::CreateNewSource, true)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(EHorizontalAlignment::HAlign_Right)
			[
				SNew(SButton)
				.Text(LOCTEXT("CancelButton", "Cancel"))
				.OnClicked(this, &SLiveLinkMasterLockitSourcePanel::CreateNewSource, false)
			]
		]
	];
}

FReply SLiveLinkMasterLockitSourcePanel::CreateNewSource(bool bShouldCreateSource)
{
	TSharedPtr<ILiveLinkSource> Source;
	FString ConnectionString;
	if (bShouldCreateSource)
	{
		const ULiveLinkMasterLockitSourceFactory* SourceFactoryPtr = SourceFactory.Get();
		if (SourceFactoryPtr)
		{
			ConnectionString = ULiveLinkMasterLockitSourceFactory::CreateConnectionString(ConnectionSettings);
			Source = SourceFactoryPtr->CreateSource(ConnectionString);
		}
	}

	OnSourceCreated.ExecuteIfBound(Source, ConnectionString);
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
