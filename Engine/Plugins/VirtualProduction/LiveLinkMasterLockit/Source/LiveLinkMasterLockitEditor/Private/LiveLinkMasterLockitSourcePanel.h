// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkMasterLockitConnectionSettings.h"

#include "LiveLinkSourceFactory.h"
#include "Widgets/SCompoundWidget.h"

class ULiveLinkMasterLockitSourceFactory;

class SLiveLinkMasterLockitSourcePanel : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkMasterLockitSourcePanel) {}
		SLATE_ARGUMENT(const ULiveLinkMasterLockitSourceFactory*, Factory)
		SLATE_EVENT(ULiveLinkSourceFactory::FOnLiveLinkSourceCreated, OnSourceCreated)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args);

public:
	FReply CreateNewSource(bool bShouldCreateSource);

private:
	FLiveLinkMasterLockitConnectionSettings ConnectionSettings;
	TWeakObjectPtr<const ULiveLinkMasterLockitSourceFactory> SourceFactory;
	ULiveLinkSourceFactory::FOnLiveLinkSourceCreated OnSourceCreated;
};
